{
	"FileVersion": 3,

	"FriendlyName": "Field Notification Trace",
	"Version": 1,
	"VersionName": "1.0",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"Description": "Add support to trace field notification object.",
	"Category": "UI",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,

	"Modules": [
		{
			"Name": "FieldNotificationTrace",
			"Type": "Runtime",
			"LoadingPhase": "Default"

		},
		{
			"Name": "FieldNotificationTraceEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],

	"Plugins": [
		{
			"Name": "GameplayInsights",
			"Enabled": true
		}
	]
}

