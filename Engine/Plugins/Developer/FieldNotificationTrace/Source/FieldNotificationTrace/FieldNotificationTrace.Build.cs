// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using System.Collections.Generic;
using UnrealBuildTool;

public class FieldNotificationTrace : ModuleRules 
{
	public FieldNotificationTrace(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"FieldNotification",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"TraceLog",
			});
			
		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"GameplayInsights",
				});
		}
	}
}
