// Copyright Epic Games, Inc. All Rights Reserved.

#include "FieldNotificationTraceModule.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FieldNotificationTrace"

namespace UE::FieldNotification
{

// void FFieldNotificationTraceModule::StartupModule()
// {

// }

// void FFieldNotificationTraceModule::ShutdownModule()
// {
// }

} //namespace 


IMPLEMENT_MODULE(UE::FieldNotification::FTraceModule, FieldNotificationTrace);

#undef LOCTEXT_NAMESPACE
