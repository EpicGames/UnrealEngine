// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

namespace UE::FieldNotification
{

/**
 *
 */
class FTraceModule : public IModuleInterface
{
public:
	FTraceModule() = default;

	//~ Begin IModuleInterface interface
	// virtual void StartupModule() override;
	// virtual void ShutdownModule() override;
	//~ End IModuleInterface interface
};

} //namespace
