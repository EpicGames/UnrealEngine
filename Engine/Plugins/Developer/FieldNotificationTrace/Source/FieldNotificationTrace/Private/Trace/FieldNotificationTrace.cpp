// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/FieldNotificationTrace.h"

#include "Misc/ScopeRWLock.h"
#include "Modules/ModuleManager.h"
#include "ObjectTrace.h"

#if UE_FIELDNOTIFICATION_TRACE_ENABLED
#define UE_FIELDNOTIFICATION_TRACE_FIELDVALUE 0


#if UE_FIELDNOTIFICATION_TRACE_FIELDVALUE
#include "IGameplayInsightsModule.h"
#endif

UE_TRACE_CHANNEL(FieldNotificationChannel);

namespace UE::FieldNotification::Private
{

bool bTraceIsRecording = false;
FAutoConsoleCommand StartTracingCommand(
	TEXT("FieldNotification.StartTracing"),
	TEXT("Turn on the recording of debugging data."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args){ FTrace::StartTracing(); })
);

FAutoConsoleCommand StopTracingCommand(
	TEXT("FieldNotification.StopTracing"),
	TEXT("Turn off the recording of debugging data."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args) { FTrace::StopTracing(); })
);

uint32 TraceFNameId(FName Name);
uint32 TraceStringId(const FString& Value);

} // namespace

namespace UE::FieldNotification
{

UE_TRACE_EVENT_BEGIN(FieldNotification, ObjectBegin)
UE_TRACE_EVENT_FIELD(uint64, Cycle)
UE_TRACE_EVENT_FIELD(uint64, ObjectId)

UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(FieldNotification, ObjectEnd)
UE_TRACE_EVENT_FIELD(uint64, Cycle)
UE_TRACE_EVENT_FIELD(uint64, ObjectId)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(FieldNotification, FieldValueChanged)
UE_TRACE_EVENT_FIELD(uint64, Cycle)
UE_TRACE_EVENT_FIELD(double, RecordingTime)
UE_TRACE_EVENT_FIELD(uint64, ObjectId)
UE_TRACE_EVENT_FIELD(uint32, FieldNotifyId)
UE_TRACE_EVENT_END()

void FTrace::OutputObjectBegin(TScriptInterface<INotifyFieldValueChanged> Interface)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(FieldNotificationChannel);
	if (!bChannelEnabled || Interface.GetObject() == nullptr)
	{
		return;
	}

	if (Interface.GetObject()->HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	TRACE_OBJECT_LIFETIME_BEGIN(Interface.GetObject());

	UE_TRACE_LOG(FieldNotification, ObjectBegin, FieldNotificationChannel)
		<< ObjectBegin.Cycle(FPlatformTime::Cycles64())
		<< ObjectBegin.ObjectId(FObjectTrace::GetObjectId(Interface.GetObject()));

	// since the world may or may not be set for those object
	// need to trace the full path of the object.
	//a viewmodel may construct other vm, so at replay we need to associate them correctly.
}

void FTrace::OutputObjectEnd(TScriptInterface<INotifyFieldValueChanged> Interface)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(FieldNotificationChannel);
	if (!bChannelEnabled || Interface.GetObject() == nullptr)
	{
		return;
	}

	if (Interface.GetObject()->HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	TRACE_OBJECT_LIFETIME_END(Interface.GetObject())

	UE_TRACE_LOG(FieldNotification, ObjectEnd, FieldNotificationChannel)
		<< ObjectEnd.Cycle(FPlatformTime::Cycles64())
		<< ObjectEnd.ObjectId(FObjectTrace::GetObjectId(Interface.GetObject()));
}

void FTrace::OutputUpdateField(UObject* Instance, UE::FieldNotification::FFieldId Id)
{
	bool bChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(FieldNotificationChannel);
	if (!bChannelEnabled || Instance == nullptr || !Id.IsValid())
	{
		return;
	}

	if (Instance->HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	TRACE_OBJECT(Instance)

	const uint64 StartCycle = FPlatformTime::Cycles64();
	const uint64 ObjectId = FObjectTrace::GetObjectId(Instance);
	const uint32 NameId = Private::TraceFNameId(Id.GetName());

#if UE_FIELDNOTIFICATION_TRACE_FIELDVALUE
	if (const FProperty* Property = Instance->GetClass()->FindPropertyByName(Id.GetName()))
	{
		IGameplayInsightsModule& GameplayInsightsModule = FModuleManager::GetModuleChecked<IGameplayInsightsModule>("GameplayInsights");
		GameplayInsightsModule.TraceObjectProperty(Instance, Property);
	}
	else
	{
		 //Serialize all the properties. In case it's a function change and the function uses properties that are not FieldNotify
		IGameplayInsightsModule& GameplayInsightsModule = FModuleManager::GetModuleChecked<IGameplayInsightsModule>("GameplayInsights");
		GameplayInsightsModule.TraceObjectProperties(Instance);
	}
#endif

	UE_TRACE_LOG(FieldNotification, FieldValueChanged, FieldNotificationChannel)
		<< FieldValueChanged.Cycle(StartCycle)
		<< FieldValueChanged.RecordingTime(FObjectTrace::GetWorldElapsedTime(Instance->GetWorld()))
		<< FieldValueChanged.ObjectId(ObjectId)
		<< FieldValueChanged.FieldNotifyId(NameId);
}

void FTrace::StartTracing()
{
	if (Private::bTraceIsRecording)
	{
		return;
	}
	Private::bTraceIsRecording = false;

	UE::Trace::ToggleChannel(TEXT("FieldNotificationChannel"), true);
	UE::Trace::ToggleChannel(TEXT("Object"), true);
#if UE_FIELDNOTIFICATION_TRACE_FIELDVALUE
	UE::Trace::ToggleChannel(TEXT("ObjectProperties"), true);
#endif
}

void FTrace::StopTracing()
{
	UE::Trace::ToggleChannel(TEXT("FieldNotificationChannel"), false);
}

} // namespace

namespace UE::FieldNotification::Private
{

UE_TRACE_EVENT_BEGIN(FieldNotification, StringId, NoSync | Important)
UE_TRACE_EVENT_FIELD(uint32, Id)
UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Value)
UE_TRACE_EVENT_END()

struct FTraceStringId
{
	uint32 CurrentId = 0;
	FRWLock SetLock;
	TMap<FName, uint32> NameToId;
	TMap<FString, uint32> StringToId;
} StringIdInstance;

uint32 TraceFNameId(FName Name)
{
	if (Name.IsNone())
	{
		return 0;
	}

	// read only mutex
	{
		FRWScopeLock ScopeLock(StringIdInstance.SetLock, SLT_ReadOnly);
		if (uint32* FoundId = StringIdInstance.NameToId.Find(Name))
		{
			return *FoundId;
		}
	}

	// write mutex
	uint32 NewId = 0;
	{
		FRWScopeLock ScopeLock(StringIdInstance.SetLock, SLT_Write);
		++StringIdInstance.CurrentId;
		NewId = StringIdInstance.CurrentId;
		StringIdInstance.NameToId.Add(Name, NewId);
	}

	FString Value = Name.ToString();
	UE_TRACE_LOG(FieldNotification, StringId, FieldNotificationChannel, Value.Len() * sizeof(TCHAR))
		<< StringId.Id(NewId)
		<< StringId.Value(*Value, Value.Len());

	return NewId;
}

uint32 TraceStringId(const FString& Value)
{
	if (Value.IsEmpty())
	{
		return 0;
	}

	// read only mutex
	{
		FRWScopeLock ScopeLock(StringIdInstance.SetLock, SLT_ReadOnly);
		if (uint32* FoundId = StringIdInstance.StringToId.Find(Value))
		{
			return *FoundId;
		}
	}

	// write mutex
	uint32 NewId = 0;
	{
		FRWScopeLock ScopeLock(StringIdInstance.SetLock, SLT_Write);
		++StringIdInstance.CurrentId;
		NewId = StringIdInstance.CurrentId;
		StringIdInstance.StringToId.Add(Value, NewId);
	}

	UE_TRACE_LOG(FieldNotification, StringId, FieldNotificationChannel, Value.Len() * sizeof(TCHAR))
		<< StringId.Id(NewId)
		<< StringId.Value(*Value, Value.Len());

	return NewId;
}

} // namespace

#endif
