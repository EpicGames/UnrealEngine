// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "Trace/Config.h"
#include "Trace/Trace.h"
#include "INotifyFieldValueChanged.h"
#include "FieldNotificationId.h"

#ifndef UE_FIELDNOTIFICATION_TRACE_ENABLED 
#define UE_FIELDNOTIFICATION_TRACE_ENABLED (UE_TRACE_ENABLED && !IS_PROGRAM && !UE_BUILD_SHIPPING)
#endif

#if UE_FIELDNOTIFICATION_TRACE_ENABLED

namespace UE::FieldNotification
{

class FTrace : public FNoncopyable
{
public:
	FIELDNOTIFICATIONTRACE_API static void OutputObjectBegin(TScriptInterface<INotifyFieldValueChanged> Interface);
	FIELDNOTIFICATIONTRACE_API static void OutputObjectEnd(TScriptInterface<INotifyFieldValueChanged> Interface);
	FIELDNOTIFICATIONTRACE_API static void OutputUpdateField(UObject* Instance, FFieldId Id);

	static void StartTracing();
	static void StopTracing();
};

} //namespace

#define UE_TRACE_FIELDNOTIFICATION_LIFETIME_BEGIN(Interface) \
::UE::FieldNotification::FTrace::OutputObjectBegin(Interface);

#define UE_TRACE_FIELDNOTIFICATION_LIFETIME_END(Interface) \
::UE::FieldNotification::FTrace::OutputObjectEnd(Interface);

#define UE_TRACE_FIELDNOTIFICATION_FIELD_VALUE_CHANGED(Object, InField) \
::UE::FieldNotification::FTrace::OutputUpdateField(Object, InField);

#else //UE_FIELDNOTIFICATION_TRACE_ENABLED

#define UE_TRACE_FIELDNOTIFICATION_LIFETIME_BEGIN(Interface)
#define UE_TRACE_FIELDNOTIFICATION_LIFETIME_END(Interface)
#define UE_TRACE_FIELDNOTIFICATION_FIELD_VALUE_CHANGED(Object, InField)

#endif //UE_FIELDNOTIFICATION_TRACE_ENABLED
