// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using System.Collections.Generic;
using UnrealBuildTool;

public class FieldNotificationTraceEditor : ModuleRules 
{
	public FieldNotificationTraceEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"FieldNotification",
				"FieldNotificationTrace",
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"GameplayInsights",
				"RewindDebuggerInterface",
				"RewindDebuggerRuntimeInterface",
				"Slate",
				"SlateCore",
				"TraceAnalysis",
				"TraceLog",
				"TraceServices",
			});
	}
}
