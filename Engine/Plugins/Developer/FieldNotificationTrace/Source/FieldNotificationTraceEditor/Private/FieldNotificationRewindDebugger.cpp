// Copyright Epic Games, Inc. All Rights Reserved.

#include "FieldNotificationRewindDebugger.h"
#include "FieldNotificationTraceProvider.h"
#include "IRewindDebugger.h"
#include "TraceServices/Model/Frames.h"

namespace UE::FieldNotification
{

void FRewindDebuggerExtension::Update(float DeltaTime, IRewindDebugger* RewindDebugger)
{
	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();

	bool bFrameFound;
	TraceServices::FFrame Frame;
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
		const TraceServices::IFrameProvider& FrameProvider = TraceServices::ReadFrameProvider(*AnalysisSession);
		bFrameFound = FrameProvider.GetFrameFromTime(TraceFrameType_Game, RewindDebugger->CurrentTraceTime(), Frame);
	}

	if (bFrameFound)
	{
		// each tick update the UMG Preview Window with what we are trying to debug
	}
}

void FRewindDebuggerRuntimeExtension::RecordingStarted()
{
	Trace::ToggleChannel(TEXT("FieldNotificationChannel"), true);
}

void FRewindDebuggerRuntimeExtension::RecordingStopped()
{
	Trace::ToggleChannel(TEXT("FieldNotificationChannel"), false);
}

} // namespace
