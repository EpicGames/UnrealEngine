// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IRewindDebuggerExtension.h"
#include "RewindDebuggerRuntimeInterface/IRewindDebuggerRuntimeExtension.h"

class IRewindDebugger;

namespace UE::FieldNotification
{

class FRewindDebuggerExtension : public IRewindDebuggerExtension
{
public:
	virtual void Update(float DeltaTime, IRewindDebugger* RewindDebugger) override;
};

//~ @todo this extension should be moved to a runtime module to support trace recording on non-editor targets
class FRewindDebuggerRuntimeExtension : public IRewindDebuggerRuntimeExtension
{
public:
	virtual void RecordingStarted() override;
	virtual void RecordingStopped() override;
};

} //namespace
