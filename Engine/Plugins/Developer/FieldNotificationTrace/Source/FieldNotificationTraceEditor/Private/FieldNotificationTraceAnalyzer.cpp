// Copyright Epic Games, Inc. All Rights Reserved.

#include "FieldNotificationTraceAnalyzer.h"

#include "FieldNotificationTraceProvider.h"
#include "HAL/LowLevelMemTracker.h"
#include "Serialization/MemoryReader.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Utils.h"

namespace UE::FieldNotification
{

FTraceAnalyzer::FTraceAnalyzer(TraceServices::IAnalysisSession& InSession, FTraceProvider& InProvider)
	: Session(InSession)
	, Provider(InProvider)
{
}

void FTraceAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	auto& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_ObjectBegin, "FieldNotification", "ObjectBegin");
	Builder.RouteEvent(RouteId_ObjectEnd, "FieldNotification", "ObjectEnd");
	Builder.RouteEvent(RouteId_FieldValueChanged, "FieldNotification", "FieldValueChanged");
	Builder.RouteEvent(RouteId_StringId, "FieldNotification", "StringId");
}

bool FTraceAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FTraceAnalyzer"));

	TraceServices::FAnalysisSessionEditScope _(Session);

	const auto& EventData = Context.EventData;
	switch (RouteId)
	{
		case RouteId_ObjectBegin:
		{
			uint64 Cycle = EventData.GetValue<uint64>("Cycle");
			uint64 ObjectId = EventData.GetValue<uint64>("ObjectId");
			Provider.AppendObjectBegin(ObjectId, Context.EventTime.AsSeconds(Cycle));
			break;
		}
		case RouteId_ObjectEnd:
		{
			uint64 Cycle = EventData.GetValue<uint64>("Cycle");
			uint64 ObjectId = EventData.GetValue<uint64>("ObjectId");
			Provider.AppendObjectEnd(ObjectId, Context.EventTime.AsSeconds(Cycle));
			break;
		}
		case RouteId_FieldValueChanged:
		{
			uint64 Cycle = EventData.GetValue<uint64>("Cycle");
			double RecordingTime = EventData.GetValue<double>("RecordingTime");
			uint64 Id = EventData.GetValue<uint64>("ObjectId");
			uint32 FieldNotifyId = EventData.GetValue<uint64>("FieldNotifyId");
			Provider.AppendFieldValueChanged(Id, Context.EventTime.AsSeconds(Cycle), RecordingTime, FieldNotifyId);
			break;
		}
		case RouteId_StringId:
		{
			uint32 Id = EventData.GetValue<uint32>("Id");
			FStringView Value;
			EventData.GetString("Value", Value);
			Provider.AppendFieldNotify(Id, FName(Value));
			break;
		}
	}

	return true;
}

} //namespace