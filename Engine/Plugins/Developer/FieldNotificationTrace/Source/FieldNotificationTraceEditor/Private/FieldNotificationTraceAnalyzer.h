// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"

namespace UE::FieldNotification { class FTraceProvider; }
namespace TraceServices { class IAnalysisSession; }

namespace UE::FieldNotification
{

class FTraceAnalyzer : public UE::Trace::IAnalyzer
{
public:
	FTraceAnalyzer(TraceServices::IAnalysisSession& InSession, FTraceProvider& InProvider);

	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override {}
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	enum : uint16
	{
		RouteId_ObjectBegin,
		RouteId_ObjectEnd,
		RouteId_FieldValueChanged,
		//RouteId_PropertyValueChanged,
		RouteId_StringId,
	};

	TraceServices::IAnalysisSession& Session;
	FTraceProvider& Provider;
};

} // namespace
