// Copyright Epic Games, Inc. All Rights Reserved.

#include "FieldNotificationTraceEditorModule.h"
#include "Modules/ModuleManager.h"

#include "Features/IModularFeatures.h"
#include "IRewindDebuggerExtension.h"

#define LOCTEXT_NAMESPACE "FieldNotificationTraceEditor"

namespace UE::FieldNotification
{

void FTraceEditorModule::StartupModule()
{
	IModularFeatures::Get().RegisterModularFeature(IRewindDebuggerExtension::ModularFeatureName, &RewindDebuggerExtension);

	//~ @todo this extension should be moved to a runtime module to support trace recording on non-editor targets
	IModularFeatures::Get().RegisterModularFeature(IRewindDebuggerRuntimeExtension::ModularFeatureName, &RewindDebuggerRuntimeExtension);
	IModularFeatures::Get().RegisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &TrackCreator);
	IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &TraceModule);
}


void FTraceEditorModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &TraceModule);
	IModularFeatures::Get().UnregisterModularFeature(RewindDebugger::IRewindDebuggerTrackCreator::ModularFeatureName, &TrackCreator);
	IModularFeatures::Get().UnregisterModularFeature(IRewindDebuggerRuntimeExtension::ModularFeatureName, &RewindDebuggerRuntimeExtension);
	IModularFeatures::Get().UnregisterModularFeature(IRewindDebuggerExtension::ModularFeatureName, &RewindDebuggerExtension);
}

} //namespace 

IMPLEMENT_MODULE(UE::FieldNotification::FTraceEditorModule, FieldNotificationTraceEditor);

#undef LOCTEXT_NAMESPACE
