// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

#include "FieldNotificationRewindDebugger.h"
#include "FieldNotificationTrack.h"
#include "FieldNotificationTraceServices.h"

namespace UE::FieldNotification
{

/**
 *
 */
class FTraceEditorModule : public IModuleInterface
{
public:
	FTraceEditorModule() = default;

	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	 virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

private:
	FRewindDebuggerExtension RewindDebuggerExtension;
	FRewindDebuggerRuntimeExtension RewindDebuggerRuntimeExtension;
#if WITH_ENGINE
	FTracksCreator TrackCreator;
#endif // WITH_ENGINE 
	FTraceServiceModule TraceModule;
};

} //namespace
