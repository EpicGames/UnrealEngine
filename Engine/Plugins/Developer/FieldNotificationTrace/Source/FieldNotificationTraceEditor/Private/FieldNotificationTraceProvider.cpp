// Copyright Epic Games, Inc. All Rights Reserved.

#include "FieldNotificationTraceProvider.h"
#include "ObjectTrace.h"

namespace UE::FieldNotification
{

FName FTraceProvider::ProviderName("FieldNotificationProvider");

#define LOCTEXT_NAMESPACE "FieldNotificationProvider"



FTraceProvider::FObject::FObject(TraceServices::IAnalysisSession& InSession)
	: FieldNotifies(InSession.GetLinearAllocator())
	, FieldNotifiesRecording(InSession.GetLinearAllocator())
{
}

FTraceProvider::FTraceProvider(TraceServices::IAnalysisSession& InSession)
	: Session(InSession)
	, ObjectLifetimes(InSession.GetLinearAllocator())
{
}

void FTraceProvider::EnumerateObjects(double StartTime, double EndTime, TFunctionRef<void(double InStartTime, double InEndTime, uint32 InDepth, const FObject&)> Callback) const
{
	ObjectLifetimes.EnumerateEvents(StartTime, EndTime,
		[this, &Callback](double InStartTime, double InEndTime, uint32 InDepth, const TSharedRef<FObject>& ExistsMessage)
		{
			Callback(InStartTime, InEndTime, InDepth, ExistsMessage.Get());
			return TraceServices::EEventEnumerate::Continue;
		});
}

void FTraceProvider::EnumerateFieldNotifies(uint64 InObjectId, double InStartTime, double InEndTime, TFunctionRef<void(double InStartTime, double InEndTime, uint32 InDepth, const FFieldNotifyEvent&)> Callback) const
{
	Session.ReadAccessCheck();

	const TSharedRef<FObject>* Found = ObjectIdToObject.Find(InObjectId);
	if (Found != nullptr)
	{
		(*Found)->FieldNotifies.EnumerateEvents(InStartTime, InEndTime,
			[&Callback](double InNewStartTime, double InNewEndTime, uint32 InDepth, const FFieldNotifyEvent& ExistsMessage)
			{
				Callback(InNewStartTime, InNewEndTime, InDepth, ExistsMessage);
				return TraceServices::EEventEnumerate::Continue;
			});
	}
}

void FTraceProvider::EnumerateRecordingFieldNotifies(uint64 InObjectId, double InStartTime, double InEndTime, TFunctionRef<void(double InStartTime, double InEndTime, uint32 InDepth, const FFieldNotifyEvent&)> Callback) const
{
	Session.ReadAccessCheck();

	const TSharedRef<FObject>* Found = ObjectIdToObject.Find(InObjectId);
	if (Found != nullptr)
	{
		(*Found)->FieldNotifiesRecording.EnumerateEvents(InStartTime, InEndTime,
			[&Callback](double InNewStartTime, double InNewEndTime, uint32 InDepth, const FFieldNotifyEvent& ExistsMessage)
			{
				Callback(InNewStartTime, InNewEndTime, InDepth, ExistsMessage);
				return TraceServices::EEventEnumerate::Continue;
			});
	}
}

FFieldNotificationId FTraceProvider::GetFieldNotificationId(uint32 InFieldNotifyId) const
{
	Session.ReadAccessCheck();
	return FFieldNotificationId(FieldNotifyIdToFieldNotifyName.FindRef(InFieldNotifyId));
}

void FTraceProvider::AppendObjectBegin(uint64 InObjectId, double InProfileTime)
{
	Session.WriteAccessCheck();

	if (!ObjectIdToObject.Contains(InObjectId))
	{
		TSharedRef<FObject> NewObject = MakeShared<FObject>(Session);
		NewObject->SelfObjectId = InObjectId;

		ObjectIdToObject.Add(InObjectId, NewObject);
		NewObject->TimelineId = ObjectLifetimes.AppendBeginEvent(InProfileTime, NewObject);
	}
}

void FTraceProvider::AppendObjectEnd(uint64 InObjectId, double InProfileTime)
{
	Session.WriteAccessCheck();

	if (TSharedRef<FObject>* Found = ObjectIdToObject.Find(InObjectId))
	{
		ObjectLifetimes.EndEvent((*Found)->TimelineId, InProfileTime);
	}
}

void FTraceProvider::AppendFieldValueChanged(uint64 InObjectId, double InProfileTime, double InRecordingTime, uint32 InFieldNotifyId)
{
	Session.WriteAccessCheck();

	FObject* CurrentObject = nullptr;
	if (TSharedRef<FObject>* Found = ObjectIdToObject.Find(InObjectId))
	{
		CurrentObject = &(Found->Get());
	}
	else
	{
		// make new object
		TSharedRef<FObject> NewObject = MakeShared<FObject>(Session);
		NewObject->SelfObjectId = InObjectId;

		ObjectIdToObject.Add(InObjectId, NewObject);
		NewObject->TimelineId = ObjectLifetimes.AppendBeginEvent(InProfileTime, NewObject);

		CurrentObject = &(NewObject.Get());
	}

	if (CurrentObject)
	{
		FFieldNotifyEvent Value;
		Value.FieldNotifyId = InFieldNotifyId;
		CurrentObject->FieldNotifies.AppendEvent(InProfileTime, Value);
		CurrentObject->FieldNotifiesRecording.AppendEvent(InRecordingTime, Value);
	}
}

void FTraceProvider::AppendFieldNotify(uint32 InFieldNotifyId, FName InValue)
{
	Session.WriteAccessCheck();

	FieldNotifyIdToFieldNotifyName.FindOrAdd(InFieldNotifyId) = InValue;
}


} //namespace

#undef LOCTEXT_NAMESPACE
