// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Engine/EngineBaseTypes.h"
#include "FieldNotificationId.h"
#include "Templates/SharedPointer.h"
#include "Model/IntervalTimeline.h"
#include "Model/PointTimeline.h"
#include "TraceServices/Model/AnalysisSession.h"

namespace TraceServices { class IAnalysisSession; }

namespace UE::FieldNotification
{

class FTraceProvider : public TraceServices::IProvider
{
public:
	struct FFieldNotifyEvent
	{
		uint32 FieldNotifyId = 0;
	};

	struct FObject
	{
		FObject(TraceServices::IAnalysisSession& InSession);
		uint64 SelfObjectId = 0;
		uint64 TimelineId = 0;
		TraceServices::TPointTimeline<FFieldNotifyEvent> FieldNotifies;
		TraceServices::TPointTimeline<FFieldNotifyEvent> FieldNotifiesRecording;
	};
	
	static FName ProviderName;

	FTraceProvider(TraceServices::IAnalysisSession& InSession);

	void EnumerateObjects(double InStartTime, double InEndTime, TFunctionRef<void(double InStartTime, double InEndTime, uint32 InDepth, const FObject&)> Callback) const;
	void EnumerateFieldNotifies(uint64 InObjectId, double InStartTime, double InEndTime, TFunctionRef<void(double InStartTime, double InEndTime, uint32 InDepth, const FFieldNotifyEvent&)> Callback) const;
	void EnumerateRecordingFieldNotifies(uint64 InObjectId, double InStartTime, double InEndTime, TFunctionRef<void(double InStartTime, double InEndTime, uint32 InDepth, const FFieldNotifyEvent&)> Callback) const;
	FFieldNotificationId GetFieldNotificationId(uint32 InFieldNotifyId) const;

	bool HasData() const
	{
		return ObjectIdToObject.Num() > 0;
	}

	bool HasData(uint64 InObjectId) const
	{
		return ObjectIdToObject.Contains(InObjectId);
	}

	void AppendObjectBegin(uint64 InObjectId, double InProfileTime);
	void AppendObjectEnd(uint64 InObjectId, double InProfileTime);
	void AppendFieldValueChanged(uint64 InObjectId, double InProfileTime, double InRecordingTime, uint32 InFieldNotifyId);
	void AppendFieldNotify(uint32 InFieldNotifyId, FName InValue);

private:
	TraceServices::IAnalysisSession& Session;

	TMap<uint32, FName> FieldNotifyIdToFieldNotifyName;
	TMap<uint64, TSharedRef<FObject>> ObjectIdToObject;

	// Timeline containing intervals where an object exists
	TraceServices::TIntervalTimeline<TSharedRef<FObject>> ObjectLifetimes;
};

} // namespace
