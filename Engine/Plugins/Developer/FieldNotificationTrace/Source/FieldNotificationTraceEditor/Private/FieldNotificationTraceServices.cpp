// Copyright Epic Games, Inc. All Rights Reserved.

#include "FieldNotificationTraceServices.h"

#include "FieldNotificationTraceAnalyzer.h"
#include "FieldNotificationTraceProvider.h"

namespace UE::FieldNotification
{

FName FTraceServiceModule::ModuleName("FieldNotification");

void FTraceServiceModule::GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo)
{
	OutModuleInfo.Name = ModuleName;
	OutModuleInfo.DisplayName = TEXT("FieldNotification");
}

void FTraceServiceModule::OnAnalysisBegin(TraceServices::IAnalysisSession& InSession)
{
	TSharedPtr<FTraceProvider> TraceProvider = MakeShared<FTraceProvider>(InSession);
	InSession.AddProvider(FTraceProvider::ProviderName, TraceProvider);

	InSession.AddAnalyzer(new FTraceAnalyzer(InSession, *TraceProvider));
}

void FTraceServiceModule::GetLoggers(TArray<const TCHAR *>& OutLoggers)
{
	OutLoggers.Add(TEXT("FieldNotification"));
}

void FTraceServiceModule::GenerateReports(const TraceServices::IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory)
{

}

} // namespace
