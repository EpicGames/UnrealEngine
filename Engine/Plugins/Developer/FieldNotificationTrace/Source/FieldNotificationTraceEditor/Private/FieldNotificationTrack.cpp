// Copyright Epic Games, Inc. All Rights Reserved.

#include "FieldNotificationTrack.h"

#include "Common/ProviderLock.h"
#include "Editor.h"
#include "FieldNotificationTraceProvider.h"
#include "IGameplayProvider.h"
#include "IRewindDebugger.h"

#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "FieldNotificationTrack"

namespace UE::FieldNotification
{

static FLinearColor MakeNotifyColor(uint32 InSeed, bool bInLine = false)
{
	FRandomStream Stream(InSeed);
	const uint8 Hue = (uint8)(Stream.FRand() * 255.0f);
	const uint8 SatVal = bInLine ? 196 : 128;
	return FLinearColor::MakeFromHSV8(Hue, SatVal, SatVal);
}

FFieldNotifyTrack::FFieldNotifyTrack(uint64 InObjectId, uint32 InFieldNotifyId, FFieldNotificationId InFieldNotify)
	: ObjectId(InObjectId)
	, FieldNotifyId(InFieldNotifyId)
	, FieldNotify(InFieldNotify)
{
	EventData = MakeShared<SEventTimelineView::FTimelineEventData>();
	Icon = FSlateIcon("EditorStyle", "Sequencer.Tracks.Event", "Sequencer.Tracks.Event");
}

TSharedPtr<SEventTimelineView::FTimelineEventData> FFieldNotifyTrack::GetEventData() const
{
	if (!EventData.IsValid())
	{
		EventData = MakeShared<SEventTimelineView::FTimelineEventData>();
	}
	
	EventUpdateRequested++;
	
	return EventData;
}

TSharedPtr<SWidget> FFieldNotifyTrack::GetDetailsViewInternal()
{
	return nullptr;
}

bool FFieldNotifyTrack::UpdateInternal()
{
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	
	TRange<double> RecordingTimeRange = RewindDebugger->GetCurrentViewRange();
	double StartTime = RecordingTimeRange.GetLowerBoundValue();
	double EndTime = RecordingTimeRange.GetUpperBoundValue();
	
	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
	const FTraceProvider* Provider = AnalysisSession->ReadProvider<FTraceProvider>(FTraceProvider::ProviderName);
	
	if(Provider)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FFieldNotifyTrack::UpdateEventPointsInternal);
		EventUpdateRequested = 0;
		
		EventData->EventPoints.SetNum(0,EAllowShrinking::No);
		EventData->EventWindows.SetNum(0);

		TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
		
		Provider->EnumerateRecordingFieldNotifies(ObjectId, StartTime, EndTime, [Self=this](double InStartTime, double InEndTime, uint32 InDepth, const FTraceProvider::FFieldNotifyEvent& EvaluationData)
			{
				if (EvaluationData.FieldNotifyId == Self->FieldNotifyId)
				{
					Self->EventData->EventPoints.Add({ InStartTime, FText(), FText(), FLinearColor::White });
				}
			});
	}

	EventUpdateRequested++;

	bool bChanged = false;
	return bChanged;
	
}

FSlateIcon FFieldNotifyTrack::GetIconInternal()
{
	return Icon;
}

FName FFieldNotifyTrack::GetNameInternal() const
{
	return FieldNotify.GetFieldName();
}

FText FFieldNotifyTrack::GetDisplayNameInternal() const
{
	return FText::FromName(FieldNotify.GetFieldName());
}

uint64 FFieldNotifyTrack::GetObjectIdInternal() const
{
	return ObjectId;
}

bool FFieldNotifyTrack::HandleDoubleClickInternal()
{
	const IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	if (const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession())
	{
		const IGameplayProvider* GameplayProvider = AnalysisSession->ReadProvider<IGameplayProvider>("GameplayProvider");
		TraceServices::FProviderReadScopeLock ProviderReadScope(*GameplayProvider);

		//const FObjectInfo& AssetInfo = GameplayProvider->GetObjectInfo(GetId());

		// attach the viewmodel to the UMG Preview Window for debugging
		//if (GetPreviewWindow)
		//{
		//	const FObjectInfo& OwnerObjectInfo  = GameplayProvider->GetObjectInfo(ObjectId);
		//	GetPreviewWindow->SetDebugTarget(OwnerObjectInfo.Name);
		//	GetPreviewWindow->bEnableDebugTesting = true;
		//}

		//GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(AssetInfo.PathName);

		return true;
	}
	return false;
}

TSharedPtr<SWidget> FFieldNotifyTrack::GetTimelineViewInternal()
{
	return SNew(SEventTimelineView)
		.ViewRange_Lambda([]() { return IRewindDebugger::Instance()->GetCurrentViewRange(); })
		.EventData_Raw(this, &FFieldNotifyTrack::GetEventData);
}

		
FObjectTrack::FObjectTrack(uint64 InObjectId)
	: ObjectId(InObjectId)
{
	Icon = FSlateIcon("EditorStyle", "Sequencer.Tracks.Event", "Sequencer.Tracks.Event");
}

TSharedPtr<SWidget> FObjectTrack::GetDetailsViewInternal()
{
	return nullptr;
}

bool FObjectTrack::UpdateInternal()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTracks::UpdateInternal);
	IRewindDebugger* RewindDebugger = IRewindDebugger::Instance();
	
	TRange<double> RecordingTimeRange = RewindDebugger->GetCurrentViewRange();
	double StartTime = RecordingTimeRange.GetLowerBoundValue();
	double EndTime = RecordingTimeRange.GetUpperBoundValue();
	
	const TraceServices::IAnalysisSession* AnalysisSession = RewindDebugger->GetAnalysisSession();
	const FTraceProvider* Provider = AnalysisSession->ReadProvider<FTraceProvider>(FTraceProvider::ProviderName);
	
	bool bChanged = false;
	
	if(Provider)
	{
		TArray<uint32> UniqueTrackIds;
		
		TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
		
		Provider->EnumerateRecordingFieldNotifies(ObjectId, StartTime, EndTime, [&UniqueTrackIds](double InStartTime, double InEndTime, uint32 InDepth, const FTraceProvider::FFieldNotifyEvent& FieldNotify)
			{
				UniqueTrackIds.AddUnique(FieldNotify.FieldNotifyId);
		 	});
		
		UniqueTrackIds.StableSort();
		const int32 TrackCount = UniqueTrackIds.Num();
		
		if (Children.Num() != TrackCount)
		{
			bChanged = true;
		}
		
		Children.SetNum(UniqueTrackIds.Num());
		for(int32 Index = 0; Index < TrackCount; ++Index)
		{
			if (!Children[Index].IsValid() || !(Children[Index].Get()->GetFieldNotifyId() == UniqueTrackIds[Index]))
			{
				Children[Index] = MakeShared<FFieldNotifyTrack>(ObjectId, UniqueTrackIds[Index], Provider->GetFieldNotificationId(UniqueTrackIds[Index]));
				bChanged = true;
			}
		
			if (Children[Index]->Update())
			{
				bChanged = true;
			}
		}
	}
	
	return bChanged;
}

FSlateIcon FObjectTrack::GetIconInternal()
{
	return Icon;
}

FName FObjectTrack::GetNameInternal() const
{
	return "FieldNotifications";
}

FText FObjectTrack::GetDisplayNameInternal() const
{
	return LOCTEXT("ObjectTrackName", "Field Notify");
}

uint64 FObjectTrack::GetObjectIdInternal() const
{
	return ObjectId;
}

TConstArrayView<TSharedPtr<RewindDebugger::FRewindDebuggerTrack>> FObjectTrack::GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const
{
	return MakeConstArrayView<TSharedPtr<FRewindDebuggerTrack>>(reinterpret_cast<const TSharedPtr<FRewindDebuggerTrack>*>(Children.GetData()), Children.Num());
}

FName FTracksCreator::GetTargetTypeNameInternal() const
{
	return UObject::StaticClass()->GetFName();
}

FName FTracksCreator::GetNameInternal() const
{
	static const FName FieldNotificationName("FieldNotification");
	return FieldNotificationName;
}

void FTracksCreator::GetTrackTypesInternal(TArray<RewindDebugger::FRewindDebuggerTrackType>& Types) const
{
	static const FName FieldNotificationName("FieldNotification");
	Types.Add({ FieldNotificationName, LOCTEXT("FieldNotification", "Field Notification") });
}

TSharedPtr<RewindDebugger::FRewindDebuggerTrack> FTracksCreator::CreateTrackInternal(const RewindDebugger::FObjectId& InObjectId) const
{
	return MakeShared<FObjectTrack>(InObjectId.GetMainId());
}

bool FTracksCreator::HasDebugInfoInternal(const RewindDebugger::FObjectId& InObjectId) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTracks::HasDebugInfoInternal);
	const TraceServices::IAnalysisSession* AnalysisSession = IRewindDebugger::Instance()->GetAnalysisSession();

	TraceServices::FAnalysisSessionReadScope SessionReadScope(*AnalysisSession);
	bool bHasData = false;
	if (const FTraceProvider* Provider = AnalysisSession->ReadProvider<FTraceProvider>(FTraceProvider::ProviderName))
	{
		bHasData = Provider->HasData(InObjectId.GetMainId());
	}
	return bHasData;
}
	
} // namespace

#undef LOCTEXT_NAMESPACE
