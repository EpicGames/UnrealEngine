// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "FieldNotificationId.h"
#if WITH_ENGINE
#include "IRewindDebuggerTrackCreator.h"
#endif // WITH_ENGINE
#include "RewindDebuggerTrack.h"
#include "Textures/SlateIcon.h"
#include "SEventTimelineView.h"
	
namespace UE::FieldNotification
{

class FFieldNotifyTrack : public RewindDebugger::FRewindDebuggerTrack
{
public:
	FFieldNotifyTrack(uint64 InObjectId, uint32 InFieldNotifyId, FFieldNotificationId InFieldNotify);
	
	uint32 GetFieldNotifyId() const
	{
		return FieldNotifyId;
	}

private:
	virtual bool UpdateInternal() override;
	TSharedPtr<SWidget> GetDetailsViewInternal() override;
	virtual TSharedPtr<SWidget> GetTimelineViewInternal() override;
	virtual FSlateIcon GetIconInternal() override;
	virtual FName GetNameInternal() const override;
	virtual FText GetDisplayNameInternal() const override;
	virtual uint64 GetObjectIdInternal() const override;
	virtual bool HandleDoubleClickInternal() override;

	TSharedPtr<SEventTimelineView::FTimelineEventData> GetEventData() const;
	
	mutable TSharedPtr<SEventTimelineView::FTimelineEventData> EventData;
	mutable int EventUpdateRequested = 0;

	uint64 ObjectId;
	uint32 FieldNotifyId;
	FFieldNotificationId FieldNotify;
	FSlateIcon Icon;
	FText TrackName;
};


class FObjectTrack : public RewindDebugger::FRewindDebuggerTrack
{
public:
	FObjectTrack(uint64 InObjectId);
private:
	virtual bool UpdateInternal() override;
	virtual TSharedPtr<SWidget> GetDetailsViewInternal() override;

	virtual FSlateIcon GetIconInternal() override;
	virtual FName GetNameInternal() const override;
	virtual FText GetDisplayNameInternal() const override;
	virtual uint64 GetObjectIdInternal() const override;
	virtual TConstArrayView<TSharedPtr<FRewindDebuggerTrack>> GetChildrenInternal(TArray<TSharedPtr<FRewindDebuggerTrack>>& OutTracks) const override;

	FSlateIcon Icon;
	uint64 ObjectId;

	TArray<TSharedPtr<FFieldNotifyTrack>> Children;
};


#if WITH_ENGINE
class FTracksCreator : public RewindDebugger::IRewindDebuggerTrackCreator
{
private:
	virtual FName GetTargetTypeNameInternal() const;
	virtual FName GetNameInternal() const override;
	virtual void GetTrackTypesInternal(TArray<RewindDebugger::FRewindDebuggerTrackType>& Types) const override;
	virtual TSharedPtr<RewindDebugger::FRewindDebuggerTrack> CreateTrackInternal(const RewindDebugger::FObjectId& InObjectId) const override;
	virtual bool HasDebugInfoInternal(const RewindDebugger::FObjectId& InObjectId) const override;
};
#endif // WITH_ENGINE

} // namespace
